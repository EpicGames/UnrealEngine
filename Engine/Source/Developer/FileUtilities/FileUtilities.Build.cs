// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class FileUtilities : ModuleRules
{
	public FileUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] { "Core" });

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(new string[] { "libzip" });
		}
	}
}
