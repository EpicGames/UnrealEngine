// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "FileUtilities/ZipArchiveWriter.h"
#include "FileUtilities/ZipArchiveReader.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/LogMacros.h"


DEFINE_LOG_CATEGORY_STATIC(LogAutomationZipArchive, Log, All);

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_ENGINE

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FZipArchiveTest, "FileUtilities.ZipArchive", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FZipArchiveTest::RunTest(const FString& InParameter)
{	
	const FString TempDir = FPaths::AutomationTransientDir();
	const FString Prefix = TEXT("ZipArchiveTest");
	const FString TxtExtension = TEXT(".txt");
	const FString TempFileToZip = FPaths::CreateTempFilename(*TempDir, *Prefix, *TxtExtension);

	// Contents to be zipped
	const FString FileContents = TEXT("FileUtilities ZipArchive Test");

	const FString ZipExtension = TEXT(".zip");
	const FString ZipFilePath = FPaths::ConvertRelativePathToFull(FPaths::CreateTempFilename(*TempDir, *Prefix, *ZipExtension));
	const FString TestDirectory = FPaths::GetPath(ZipFilePath);

	// Make sure the directory where OpenWrite is called exists
	const bool bMakeTree = true;
	UTEST_TRUE("Making directory tree", IFileManager::Get().MakeDirectory(*TestDirectory, bMakeTree));

	ON_SCOPE_EXIT
	{
		// Make sure the Tmp folder gets deleted when the tests finishes
		const bool bRequireExists = true;
		const bool bRemoveTree = true;
		IFileManager::Get().DeleteDirectory(*TestDirectory, bRequireExists, bRemoveTree);
	};

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	
	{
		// Creates a zip file

		IFileHandle* ZipFile = PlatformFile.OpenWrite(*ZipFilePath);
		UTEST_NOT_NULL("Zip File is valid", ZipFile);

		FZipArchiveWriter ZipWriter{ ZipFile };

		auto ANSIFileContents = StringCast<ANSICHAR>(*FileContents);
		TConstArrayView<uint8> StringView((uint8*) ANSIFileContents.Get(), ANSIFileContents.Length());
		ZipWriter.AddFile(FPaths::GetCleanFilename(TempFileToZip), StringView, FDateTime::Now());

	}

#if WITH_EDITOR

	// FZipArchiveReader is editor only

	{
		// Reads the zip file and see if the contents are correct

		IFileHandle* ZipFile = PlatformFile.OpenRead(*ZipFilePath);
		UTEST_NOT_NULL("Zip File is valid", ZipFile);

		FZipArchiveReader ZipReader{ ZipFile };
		const TArray<FString> FileNames = ZipReader.GetFileNames();
		UTEST_EQUAL("File Count", FileNames.Num(), 1);

		for (const FString& FileName : FileNames)
		{
			TArray<uint8> FileContentsBuffer;
			UTEST_TRUE("Try Read File From Zip", ZipReader.TryReadFile(FileName, FileContentsBuffer));

			TConstArrayView<ANSICHAR> StringView((ANSICHAR*) FileContentsBuffer.GetData(), FileContentsBuffer.Num());
			UTEST_EQUAL("Are Contents the Same", FString{ StringView }, FileContents);
		}
	}

#endif // WITH_EDITOR
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FZipArchiveCompressTest, "FileUtilities.ZipArchiveCompress", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FZipArchiveCompressTest::RunTest(const FString& InParameter)
{
	IFileManager& FileManager = IFileManager::Get();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FString FileExt = TEXT("");
	FString FileDir = TEXT("");
	bool bCompress = false;
	FString CmdLine = FCommandLine::Get();
	FParse::Value(*CmdLine, TEXT("zipArchiveExt="), FileExt, false);
	FParse::Value(*CmdLine, TEXT("zipArchiveDir="), FileDir, false);
	bCompress = FParse::Param(FCommandLine::Get(), TEXT("zipArchiveCompress"));

	// Didn't find any command line arguments, exit the test.
	if (FileExt.IsEmpty() || FileDir.IsEmpty())
	{
		return true;
	}

	const FString TestDirectory = FPaths::AutomationTransientDir();
	PlatformFile.CreateDirectory(*TestDirectory);

	const FString ZipFilePath = FPaths::ConvertRelativePathToFull(FPaths::CreateTempFilename(*TestDirectory, TEXT("ZipArchiveCompressTest"), TEXT(".zip")));

	// Make sure the directory where OpenWrite is called exists
	const bool bMakeTree = true;
	UTEST_TRUE("Making directory tree", IFileManager::Get().MakeDirectory(*TestDirectory, bMakeTree));

	ON_SCOPE_EXIT
	{
		// Make sure the Tmp folder gets deleted when the tests finishes
		const bool bRequireExists = true;
		const bool bRemoveTree = true;
		FileManager.DeleteDirectory(*TestDirectory, bRequireExists, bRemoveTree);
	};

	// create the compress zip file
	{
		// Find all the files in the directory
		TArray<FString> FileToCompress;
		PlatformFile.FindFiles(FileToCompress, *FileDir, *FileExt);

		// Creates a zip file
		IFileHandle* ZipFile = PlatformFile.OpenWrite(*ZipFilePath);
		UTEST_NOT_NULL("Zip File is valid", ZipFile);

		EZipArchiveOptions ZipOptions = EZipArchiveOptions::RemoveDuplicate;
		if (bCompress)
		{
			ZipOptions |= EZipArchiveOptions::Deflate;
		}

		FZipArchiveWriter ZipWriter(ZipFile, ZipOptions);

		for (int Index = 0; Index < FileToCompress.Num(); Index++)
		{
			TUniquePtr<FArchive> Reader = TUniquePtr<FArchive>(FileManager.CreateFileReader(*FileToCompress[Index]));
			UTEST_TRUE("Fail to create the File archive reader", Reader.IsValid());

			int64 Size = Reader->TotalSize();
			TArray<uint8> RawData;
			RawData.AddUninitialized(Size);
			Reader->Serialize(RawData.GetData(), Size);
			Reader->Close();
			ZipWriter.AddFile(FileToCompress[Index], RawData, FDateTime::Now());
		}
	}

	// Reads the zip file and see if the contents are correct
	{
		IFileHandle* ZipFile = PlatformFile.OpenRead(*ZipFilePath);
		UTEST_NOT_NULL("Zip File is valid", ZipFile);

		FZipArchiveReader ZipReader{ ZipFile };
		const TArray<FString> FileNames = ZipReader.GetFileNames();
		int32 NbProcessFiles = 0;
		for (const FString& FileName : FileNames)
		{
			// open the source uncompress file 
			UTEST_TRUE("File name doesn't match", FileManager.FileExists(*FileName));

			TUniquePtr<FArchive> UnCompressReader = TUniquePtr<FArchive>(FileManager.CreateFileReader(*FileName));
			UTEST_TRUE("Fail read the uncompress file", UnCompressReader.IsValid());

			// extract the file in the zip archive
			TArray<uint8> CompressData;
			UTEST_TRUE("Try Read File From Zip", ZipReader.TryReadFile(FileName, CompressData));

			TConstArrayView<ANSICHAR> CompressDataView((ANSICHAR*)CompressData.GetData(), CompressData.Num());
			int64 Size = UnCompressReader->TotalSize();
			TArray<uint8> UnCompressData;
			UnCompressData.AddUninitialized(Size);
			UnCompressReader->Serialize(UnCompressData.GetData(), Size);
			UnCompressReader->Close();
			TConstArrayView<ANSICHAR> UnCompressView((ANSICHAR*)UnCompressData.GetData(), UnCompressData.Num());

			FString StringCompressView(CompressDataView);
			FString StringUnCompressView(CompressDataView);
			UTEST_EQUAL("Are Contents the Same", FString{ CompressDataView }, FString{ UnCompressView });
			NbProcessFiles++;
		}
		if (NbProcessFiles == FileNames.Num())
		{
			UE_LOGF(LogAutomationZipArchive, Display, "Test succeded");
		}
	}

	return true;
}

#endif // WITH_ENGINE

#endif // WITH_DEV_AUTOMATION_TESTS