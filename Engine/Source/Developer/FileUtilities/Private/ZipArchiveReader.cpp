// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileUtilities/ZipArchiveReader.h"

#if WITH_EDITOR

#include "Containers/StringConv.h"
#include "Containers/Map.h"
#include "FileUtilities/ZipFileMetaData.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "libzip/zip.h"
#include "Logging/LogMacros.h"
#include "Math/NumericLimits.h"
#include "Misc/OutputDevice.h"
#include "ZipArchivePrivate.h"

class FZipArchiveReader::FImpl
{
private:
	TMap<FString, zip_int64_t> EmbeddedFileToIndex;
	IFileHandle* FileHandle = nullptr;
	zip_source_t* ZipFileSource = nullptr;
	zip_t* ZipFile = nullptr;
	uint64 FilePos = 0;
	uint64 FileSize = 0;

	void Destruct();
	zip_int64_t ZipSourceFunctionReader(void* OutData, zip_uint64_t DataLen, zip_source_cmd_t Command);

	static zip_int64_t ZipSourceFunctionReaderStatic(void* InUserData, void* OutData, zip_uint64_t DataLen,
		zip_source_cmd_t Command);

public:
	FImpl(IFileHandle* InFileHandle, FOutputDevice* ErrorHandler);
	~FImpl();

	bool IsValid() const;
	TArray<FString> GetFileNames() const;
	bool TryReadFile(FStringView FileName, TArray<uint8>& OutData, FOutputDevice* ErrorHandler, UE::FileUtilities::FZipFileMetaData* OutMetadata) const;
};


FZipArchiveReader::FZipArchiveReader(IFileHandle* InFileHandle, FOutputDevice* ErrorHandler)
	: Impl(MakeUnique<FZipArchiveReader::FImpl>(InFileHandle, ErrorHandler))
{
}

// Defined in CPP so that TUniquePtr destructor has access to FImpl definition.
FZipArchiveReader::~FZipArchiveReader() = default;

bool FZipArchiveReader::IsValid() const
{
	return Impl->IsValid();
}

TArray<FString> FZipArchiveReader::GetFileNames() const
{
	return Impl->GetFileNames();
}

bool FZipArchiveReader::TryReadFile(
	FStringView FileName, TArray<uint8>& OutData, FOutputDevice* ErrorHandler, UE::FileUtilities::FZipFileMetaData* OutMetadata
	) const
{
	return Impl->TryReadFile(FileName, OutData, ErrorHandler, OutMetadata);
}

FZipArchiveReader::FImpl::FImpl(IFileHandle* InFileHandle, FOutputDevice* ErrorHandler)
	: FileHandle(InFileHandle)
{
	if (!FileHandle)
	{
		Destruct();
		return;
	}

	if (FileHandle->Tell() != 0)
	{
		FileHandle->Seek(0);
	}
	FilePos = 0;
	FileSize = FileHandle->Size();
	zip_error_t ZipError;
	zip_error_init(&ZipError);
	ZipFileSource = zip_source_function_create(ZipSourceFunctionReaderStatic, this, &ZipError);
	if (!ZipFileSource)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				FString::Printf(TEXT("Could not create ZipSourceFunction: %hs"), zip_error_strerror(&ZipError)));
		}
		zip_error_fini(&ZipError);
		Destruct();
		return;
	}

	zip_error_init(&ZipError);
	ZipFile = zip_open_from_source(ZipFileSource, ZIP_RDONLY, &ZipError);
	if (!ZipFile)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				FString::Printf(TEXT("Could not parse zip file: %hs"), zip_error_strerror(&ZipError)));
		}
		zip_error_fini(&ZipError);
		Destruct();
		return;
	}

	zip_int64_t NumberOfFiles = zip_get_num_entries(ZipFile, 0);
	if (NumberOfFiles < 0 || MAX_int32 < NumberOfFiles)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				TEXT("Invalid number of embedded files in the zip."));
		}
		Destruct();
		return;
	}
	EmbeddedFileToIndex.Reserve(NumberOfFiles);

	// produce the manifest file first in case the operation gets canceled while unzipping
	for (zip_int64_t i = 0; i < NumberOfFiles; i++)
	{
		zip_stat_t ZipFileStat;
		if (zip_stat_index(ZipFile, i, 0, &ZipFileStat) != 0)
		{
			if (ErrorHandler)
			{
				ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
					TEXT("Could not get stat for embedded file."));
			}
			Destruct();
			return;
		}
		zip_uint64_t ValidStat = ZipFileStat.valid;
		if (!(ValidStat & ZIP_STAT_NAME))
		{
			if (ErrorHandler)
			{
				ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
					TEXT("Stat for embedded file does not include name."));
			}
			Destruct();
			return;
		}
		EmbeddedFileToIndex.Add(FString(ANSI_TO_TCHAR(ZipFileStat.name)), i);
	}
}

FZipArchiveReader::FImpl::~FImpl()
{
	Destruct();
}

void FZipArchiveReader::FImpl::Destruct()
{
	EmbeddedFileToIndex.Empty();
	if (ZipFile)
	{
		zip_close(ZipFile);
		ZipFile = nullptr;
	}
	if (ZipFileSource)
	{
		zip_source_close(ZipFileSource);
		ZipFileSource = nullptr;
	}
	delete FileHandle;
	FileHandle = nullptr;
}

bool FZipArchiveReader::FImpl::IsValid() const
{
	return ZipFile != nullptr;
}

TArray<FString> FZipArchiveReader::FImpl::GetFileNames() const
{
	TArray<FString> Result;
	EmbeddedFileToIndex.GenerateKeyArray(Result);
	return Result;
}

bool FZipArchiveReader::FImpl::TryReadFile(
	FStringView FileName, TArray<uint8>& OutData, FOutputDevice* ErrorHandler, UE::FileUtilities::FZipFileMetaData* OutMetadata
	) const
{
	OutData.Reset();

	const zip_int64_t* Index = EmbeddedFileToIndex.FindByHash(GetTypeHash(FileName), FileName);
	if (!Index)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				FString::Printf(TEXT("File %.*s was not found in the zip file's list of embedded files."), 
					FileName.Len(), FileName.GetData()));
		}
		return false;
	}

	zip_stat_t ZipFileStat;
	if (zip_stat_index(ZipFile, *Index, 0, &ZipFileStat) != 0)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				TEXT("Could not get stat for embedded file."));
		}
		return false;
	}

	if (!(ZipFileStat.valid & ZIP_STAT_SIZE))
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				TEXT("Stat for embedded file does not include size."));
		}
		return false;
	}

	if (ZipFileStat.size == 0)
	{
		return true;
	}
	if (ZipFileStat.size > MAX_int32)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				FString::Printf(TEXT("Embedded file %.*s has size %" UINT64_FMT " which is too large to store in a TArray."),
					FileName.Len(), FileName.GetData(), ZipFileStat.size));
		}
		return false;
	}

	OutData.SetNumUninitialized(ZipFileStat.size, EAllowShrinking::No);

	zip_file* EmbeddedFile = zip_fopen_index(ZipFile, *Index, 0 /* flags */);
	if (!EmbeddedFile)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				TEXT("zip_fopen_index failed."));
		}
		OutData.Reset();
		return false;
	}
	bool bReadSuccess = zip_fread(EmbeddedFile, OutData.GetData(), ZipFileStat.size) == ZipFileStat.size;
	zip_fclose(EmbeddedFile);
	if (!bReadSuccess)
	{
		if (ErrorHandler)
		{
			ErrorHandler->Log(LogZipArchive.GetCategoryName(), ELogVerbosity::Display,
				TEXT("zip_fread failed."));
		}
		OutData.Reset();
		return false;
	}
	
	if (OutMetadata && ZipFileStat.valid & ZIP_STAT_MTIME)
	{
		OutMetadata->Timestamp = FDateTime::FromUnixTimestamp(ZipFileStat.mtime);
	}

	return true;
}

zip_int64_t FZipArchiveReader::FImpl::ZipSourceFunctionReaderStatic(
	void* InUserData, void* OutData, zip_uint64_t DataLen, zip_source_cmd_t Command)
{
	return reinterpret_cast<FZipArchiveReader::FImpl*>(InUserData)->ZipSourceFunctionReader(
		OutData, DataLen, Command);
}

zip_int64_t FZipArchiveReader::FImpl::ZipSourceFunctionReader(
	void* OutData, zip_uint64_t DataLen, zip_source_cmd_t Command)
{
	switch (Command)
	{
	case ZIP_SOURCE_OPEN:
		return 0;
	case ZIP_SOURCE_READ:
		if (FilePos == FileSize)
		{
			return 0;
		}
		DataLen = FMath::Min(static_cast<zip_uint64_t>(FileSize - FilePos), DataLen);
		if (!FileHandle->Read(reinterpret_cast<uint8*>(OutData), DataLen))
		{
			return 0;
		}
		FilePos += DataLen;
		return DataLen;
	case ZIP_SOURCE_CLOSE:
		return 0;
	case ZIP_SOURCE_STAT:
	{
		zip_stat_t* OutStat = reinterpret_cast<zip_stat_t*>(OutData);
		zip_stat_init(OutStat);
		OutStat->size = FileSize;
		OutStat->comp_size = FileSize;
		OutStat->comp_method = ZIP_CM_STORE;
		OutStat->encryption_method = ZIP_EM_NONE;
		OutStat->valid = ZIP_STAT_SIZE | ZIP_STAT_COMP_SIZE | ZIP_STAT_COMP_METHOD | ZIP_STAT_ENCRYPTION_METHOD;
		return sizeof(*OutStat);
	}
	case ZIP_SOURCE_ERROR:
	{
		zip_uint32_t* OutLibZipError = reinterpret_cast<zip_uint32_t*>(OutData);
		zip_uint32_t* OutSystemError = OutLibZipError + 1;
		*OutLibZipError = ZIP_ER_INTERNAL;
		*OutSystemError = 0;
		return 2 * sizeof(*OutLibZipError);
	}
	case ZIP_SOURCE_FREE:
		return 0;
	case ZIP_SOURCE_SEEK:
	{
		zip_int64_t NewOffset = zip_source_seek_compute_offset(FilePos, FileSize, OutData, DataLen, nullptr);
		if (NewOffset < 0 || FileSize < static_cast<uint64>(NewOffset))
		{
			return -1;
		}

		if (!FileHandle->Seek(NewOffset))
		{
			return -1;
		}
		FilePos = NewOffset;
		return 0;
	}
	case ZIP_SOURCE_TELL:
		return static_cast<zip_int64_t>(FilePos);
	case ZIP_SOURCE_SUPPORTS:
		return zip_source_make_command_bitmap(ZIP_SOURCE_OPEN, ZIP_SOURCE_READ, ZIP_SOURCE_CLOSE, ZIP_SOURCE_STAT,
			ZIP_SOURCE_ERROR, ZIP_SOURCE_FREE, ZIP_SOURCE_SEEK, ZIP_SOURCE_TELL, ZIP_SOURCE_SUPPORTS, -1);
	default:
		return 0;
	}
}

#endif