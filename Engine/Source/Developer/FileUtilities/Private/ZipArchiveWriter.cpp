// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileUtilities/ZipArchiveWriter.h"

#if WITH_ENGINE

#include "Containers/Utf8String.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "libzip/zip.h"
#include "Misc/Compression.h"
#include "ZipArchivePrivate.h"


DEFINE_LOG_CATEGORY(LogZipArchive);

// Zip File Format Specification (We dont use encryption and data descriptor)
// https://www.loc.gov/preservation/digital/formats/digformatspecs/APPNOTE%2820120901%29_Version_6.3.3.txt
//
// Zip64 Extended Information block
#pragma pack(push,1)
struct FZip64ExtraFieldHeader
{
	uint16 Id = 0x0001; // Tag the Zip64
	uint16 Length = 16; // Size of the data (16 bytes = 2 x uint64)
	uint64 UnCompressedSize;
	uint64 CompressedSize;
};
#pragma pack(pop)

#pragma pack(push,1)
struct FZipCDHeaderExtraFields : public FZip64ExtraFieldHeader
{
	uint64 Offset; // ByteOffset from beginning of zipfile of the bytes of the FileEntry
};
#pragma pack(pop)

enum class EZipGenPurposeFlags : uint16
{
	LanguageEncodingFlag = 1 << 11
};

// Local File Header
#pragma pack(push,1)
struct FZipLocalHeader
{
	uint8 Sig[4] = { 0x50, 0x4b, 0x03, 0x04 }; // Local file header signature
	uint16 Version = 45; // Version needed to extract (MS DOS - v4.5)
	uint16 GenPurposeBit; // General purpose bit flag (Language encoding flag = 1)
	uint16 CompMode;
	uint32 TimeDate;
	uint32 Crc;
	uint32 CompressedSize; // 0xFFFFFFFF if Zip64 format
	uint32 UnCompressedSize; // 0xFFFFFFFF if Zip64 format
	uint16 FileNameLength;
	uint16 ExtraFieldLength;// if zip64Format = sizeof(FZip64ExtraFieldHeader); 

};
#pragma pack(pop)

enum class EZipExternalFileAttributeFlags : uint32
{
	MSDOS_Directory_Archive = 1 << 5
};

// Central directory header
#pragma pack(push,1)
struct FZipCDHeader
{
	uint8 Sig[4] = { 0x50, 0x4b, 0x01, 0x02 }; // Central file header signature
	uint16 VersionMade = 63; // Version made by (MS-DOS - v6.3)
	uint16 VersionNeeded = 45; // Version needed to extract (MS-DOS - v4.5)
	uint16 GenPurposeBit; // General purpose bit flag
	uint16 CompMode;
	uint32 TimeDate;
	uint32 Crc;
	uint32 CompressedSize; // 0xFFFFFFFF if Zip64 format
	uint32 UnCompressedSize; // 0xFFFFFFFF if Zip64 format
	uint16 FilenameLength;
	uint16 ExtraFieldLength;// = sizeof(FZip64ExtraFieldHeader);// Length of extra fields (Zip64 Extended Information)
	uint16 FileCommentLength = 0;
	uint16 DiskNumberStart = 0;
	uint16 InternalFileAttr = 0;
	uint32 ExternalFileAttr = (uint32)EZipExternalFileAttributeFlags::MSDOS_Directory_Archive;
	uint32 RelativeLocHeaderOffset;// 0xFFFFFFFF if Zip64 format 
};
#pragma pack(pop)

// ZIP64 end of central directory record header
#pragma pack(push,1)
struct FZip64EndOfCDRecord
{
	uint8 Sig[4] = { 0x50, 0x4b, 0x06, 0x06 }; // Zip64 end of central directory record signature
	uint64 SizeOfEndOfCDR = 0x2c; // Size of the end of central directory record
	uint16 VersionMade = 63; // Version made by (MS-DOS - v6.3)
	uint16 VersionNeeded = 45; // Version Viewer (MS-DOS - v4.5)
	uint32 DiskNumber = 0;
	uint32 CDDiskNumber = 0;
	uint64 CDRecords;
	uint64 CDTotalRecords;
	uint64 CDSize;
	uint64 CDStartOffset;
};
#pragma pack(pop)

//  ZIP64 end of central directory locator header
#pragma pack(push,1)
struct FZip64EndOfCDLocator
{
	uint8 Sig[4] = { 0x50, 0x4b, 0x06, 0x07 }; // Zip64 end of central directory locator signature
	uint32 DiskNumber = 0; // Disk with end of central directory record
	uint64 EndOffsetCD; // Offset of end of central directory
	uint32 TotalDiskNumber = 1;
};
#pragma pack(pop)

// Legacy(non - ZIP64) header for the End of Central Directory
#pragma pack(push,1)
struct FZipEndOfCDRecord
{
	uint8 Sig[4] = { 0x50, 0x4b, 0x05, 0x06 }; // End of central directory record signature
	uint16 DiskNumber = 0xFFFF; //0xFFFF means read the value from the FZip64EndOfCDRecord
	uint16 CDDiskNumber = 0xFFFF; //0xFFFF means read the value from the FZip64EndOfCDRecord
	uint16 CDRecords = 0xFFFF; //0xFFFF means read the value from the FZip64EndOfCDRecord
	uint16 TotalCDRecords = 0xFFFF; //0xFFFF means read the value from the FZip64EndOfCDRecord
	uint32 CDSize = 0xFFFFFFFF; //0xFFFF means read the value from the FZip64EndOfCDRecord
	uint32 CDOffset = 0xFFFFFFFF; //0xFFFF means read the value from the FZip64EndOfCDRecord
	uint16 CommentLength = 0;
};
#pragma pack(pop)

FZipArchiveWriter::FZipArchiveWriter(IFileHandle* InFile, EZipArchiveOptions InZipOptions)
	: File(InFile),
	ZipOptions(InZipOptions)
{
}

FZipArchiveWriter::~FZipArchiveWriter()
{
	UE_LOGF(LogZipArchive, Display, "Closing zip file with %d entries.", Files.Num());

	// Write the file central directory
	uint64 DirStartOffset = Tell();

	for (FFileEntry& Entry : Files)
	{
		FUtf8String UTF8Filename = *Entry.Filename;
		///////////////////////////
		// central directory header
		///////////////////////////
		FZipCDHeader ZipCDHeader;
		ZipCDHeader.GenPurposeBit = (uint16)EZipGenPurposeFlags::LanguageEncodingFlag;
		ZipCDHeader.CompMode = Entry.bIsCompress ? ZIP_CM_DEFLATE : ZIP_CM_STORE;
		ZipCDHeader.TimeDate = Entry.Time;
		ZipCDHeader.Crc = Entry.Crc32;
		ZipCDHeader.UnCompressedSize = 0xFFFFFFFF;
		ZipCDHeader.CompressedSize = 0xFFFFFFFF;
		ZipCDHeader.FilenameLength = UTF8Filename.Len();
		ZipCDHeader.ExtraFieldLength = sizeof(FZipCDHeaderExtraFields);
		ZipCDHeader.RelativeLocHeaderOffset = 0xFFFFFFFF;
		Write((void*)&ZipCDHeader, sizeof(FZipCDHeader));

		// Write the variable size data of central directory
		Write((void*)GetData(UTF8Filename), UTF8Filename.Len());

		///////////////////////////
		// Zip64 Extended Information block
		///////////////////////////
		FZipCDHeaderExtraFields Zip64ExtraField;
		Zip64ExtraField.Length = sizeof(Zip64ExtraField) - sizeof(Zip64ExtraField.Id) - sizeof(Zip64ExtraField.Length); // Length of the data after the Length field
		Zip64ExtraField.UnCompressedSize = Entry.UnCompressedSize;
		Zip64ExtraField.CompressedSize = Entry.CompressedSize;
		Zip64ExtraField.Offset = Entry.Offset;
		Write((void*)&Zip64ExtraField, sizeof(FZipCDHeaderExtraFields));
		Flush();
	}

	uint64_t EndOffsetCD = Tell();
	uint64_t DirectorySizeInBytes = EndOffsetCD - DirStartOffset;


	///////////////////////////
	// Write ZIP64 end of central directory record
	///////////////////////////
	FZip64EndOfCDRecord Zip64EndOfCDRecord;
	Zip64EndOfCDRecord.CDRecords = Files.Num();
	Zip64EndOfCDRecord.CDTotalRecords = Files.Num();
	Zip64EndOfCDRecord.CDSize = DirectorySizeInBytes;
	Zip64EndOfCDRecord.CDStartOffset = DirStartOffset;
	Write((void*)&Zip64EndOfCDRecord, sizeof(FZip64EndOfCDRecord));

	///////////////////////////
	// Write ZIP64 end of central directory locator
	///////////////////////////
	FZip64EndOfCDLocator Zip64EndOfCDLocator;
	Zip64EndOfCDLocator.EndOffsetCD = EndOffsetCD;
	Write((void*)&Zip64EndOfCDLocator, sizeof(FZip64EndOfCDLocator));

	///////////////////////////
	// Write regular end of central directory locator
	///////////////////////////
	FZipEndOfCDRecord ZipEndOfCDRecord;
	Write((void*)&ZipEndOfCDRecord, sizeof(FZipEndOfCDRecord));

	Flush();

	if (File)
	{
		// Close the file
		delete File;
		File = nullptr;
	}
}

void FZipArchiveWriter::AddFile(const FString& Filename, TConstArrayView<uint8> Data, const FDateTime& Timestamp)
{
	if (!ensureMsgf(!Filename.IsEmpty(), TEXT("Failed to write data to zip file; filename is empty.")))
	{
		return;
	}

	if (EnumHasAnyFlags(ZipOptions, EZipArchiveOptions::RemoveDuplicate))
	{
		if (Files.IndexOfByPredicate([&SearchName = Filename](const FFileEntry& Entry) { return Entry.Filename == SearchName; }) != INDEX_NONE)
		{
			return;
		}
	}

	uint32 Crc = FCrc::MemCrc32(Data.GetData(), Data.Num());

	// Convert the date-time to a zip file timestamp (2-second resolution).
	uint32 ZipTime =
		(Timestamp.GetSecond() / 2) |
		(Timestamp.GetMinute() << 5) |
		(Timestamp.GetHour() << 11) |
		(Timestamp.GetDay() << 16) |
		(Timestamp.GetMonth() << 21) |
		((Timestamp.GetYear() - 1980) << 25);

	uint64 FileOffset = Tell();

	TArray<uint8> TempCompressedData;
	int32 OriginalSize = Data.Num();
	int32 FinalDataSize = Data.Num();
	void* WriteData = (void*)Data.GetData();
	bool bCompressionIsValid = false;

	// Compress the data 
	if (EnumHasAnyFlags(ZipOptions, EZipArchiveOptions::Deflate))
	{
		// pre allocate a buffer for the compression result
		int32 CompressedSize = FCompression::CompressMemoryBound(NAME_Zlib, OriginalSize, COMPRESS_NoFlags);
		TempCompressedData.AddUninitialized(CompressedSize);
		bool bSuccess = FCompression::CompressMemory(
			NAME_Zlib,
			TempCompressedData.GetData(),
			CompressedSize,
			WriteData,
			OriginalSize,
			COMPRESS_NoFlags,
			// Flags passed into zlib's defalteInit2 function.
			// From zlib.net/manual.html:
			// windowBits can also be –8..–15 for raw deflate. In this case, -windowBits determines the window size. 
			// deflate() will then  generate raw deflate data with no zlib header or trailer, and will not compute a check value.
			-DEFAULT_ZLIB_BIT_WINDOW);

		if (bSuccess)
		{
			WriteData = (void*)TempCompressedData.GetData();
			bCompressionIsValid = true;
			FinalDataSize = CompressedSize;
		}
	}

	FFileEntry* Entry = new (Files) FFileEntry(Filename, Crc, FileOffset, ZipTime);
	Entry->CompressedSize = FinalDataSize;
	Entry->UnCompressedSize = OriginalSize;
	Entry->bIsCompress = bCompressionIsValid;

	///////////////////////////
	// Local File Header
	///////////////////////////
	FUtf8String UTF8Filename = *Entry->Filename;
	FZipLocalHeader ZipLocalHeader;
	ZipLocalHeader.GenPurposeBit = (uint16)EZipGenPurposeFlags::LanguageEncodingFlag;
	ZipLocalHeader.CompMode = bCompressionIsValid ? ZIP_CM_DEFLATE : ZIP_CM_STORE;
	ZipLocalHeader.TimeDate = ZipTime;
	ZipLocalHeader.Crc = Crc;
	ZipLocalHeader.CompressedSize = 0xFFFFFFFF; // Set to 0xff as it is provided by the Zip64 block
	ZipLocalHeader.UnCompressedSize = 0xFFFFFFFF; // Set to 0xff as it is provided by the Zip64 block	
	ZipLocalHeader.FileNameLength = UTF8Filename.Len();
	ZipLocalHeader.ExtraFieldLength = sizeof(FZip64ExtraFieldHeader);
	Write((void*)&ZipLocalHeader, sizeof(FZipLocalHeader));
	// write the variable size data
	Write((void*)GetData(UTF8Filename), UTF8Filename.Len());

	///////////////////////////
	// Zip64 Extended Information block
	///////////////////////////
	FZip64ExtraFieldHeader Zip64ExtraFieldHeader;
	Zip64ExtraFieldHeader.UnCompressedSize = OriginalSize;
	Zip64ExtraFieldHeader.CompressedSize = FinalDataSize;
	Write((void*)&Zip64ExtraFieldHeader, sizeof(FZip64ExtraFieldHeader));

	///////////////////////////
	// Write the file data, either compressed or uncompressed as set above
	///////////////////////////
	Write((void*)WriteData, FinalDataSize);
	Flush();
}

void FZipArchiveWriter::AddFile(const FString& Filename, const TArray<uint8>& Data, const FDateTime& Timestamp)
{
	AddFile(Filename, TConstArrayView<uint8>(Data), Timestamp);
}

void FZipArchiveWriter::Flush()
{
	if (Buffer.Num())
	{
		if (File && !File->Write(Buffer.GetData(), Buffer.Num()))
		{
			UE_LOGF(LogZipArchive, Error, "Failed to write to zip file. Zip file writing aborted.");
			delete File;
			File = nullptr;
		}

		Buffer.Reset(Buffer.Num());
	}
}

#endif
