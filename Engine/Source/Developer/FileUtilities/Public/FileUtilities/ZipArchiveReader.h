// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Templates/UniquePtr.h"

namespace UE::FileUtilities
{
struct FZipFileMetaData;
}

class FOutputDevice;
class IFileHandle;

/** Helper class for reading an uncompressed zip archive file. */
class FZipArchiveReader
{
	class FImpl;
	TUniquePtr<FImpl> Impl;

public:
	/**
	 * Constructs the ZipArchive to read from a FileHandle. Takes ownership of the FileHandle
	 * (even if it is corrupt); it will be closed by the destructor.
	 */
	FILEUTILITIES_API FZipArchiveReader(IFileHandle* InFileHandle, FOutputDevice* ErrorHandler = nullptr);
	FILEUTILITIES_API ~FZipArchiveReader();

	FILEUTILITIES_API bool IsValid() const;

	FILEUTILITIES_API TArray<FString> GetFileNames() const;
	FILEUTILITIES_API bool TryReadFile(FStringView FileName, TArray<uint8>& OutData, 
		FOutputDevice* ErrorHandler = nullptr, 
		UE::FileUtilities::FZipFileMetaData* OutMetadata = nullptr
		) const;
};

#endif
