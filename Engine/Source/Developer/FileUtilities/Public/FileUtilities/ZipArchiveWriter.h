// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Misc/DateTime.h"
 #include "Misc/EnumClassFlags.h"

#define UE_API FILEUTILITIES_API
 
#if WITH_ENGINE

class IFileHandle;

enum class EZipArchiveOptions : uint8
{
	None			= 0,
	Deflate			= 1 << 0,
	RemoveDuplicate = 1 << 1
};
ENUM_CLASS_FLAGS(EZipArchiveOptions);

/** Helper class for generating an uncompressed zip archive file. */
class FZipArchiveWriter
{
	struct FFileEntry
	{
		FString Filename;
		uint32 Crc32;
		uint64 Offset;
		uint32 Time;
		uint64 CompressedSize;
		uint64 UnCompressedSize;
		bool bIsCompress;

		FFileEntry(const FString& InFilename, uint32 InCrc32, uint64 InOffset, uint32 InTime)
			: Filename(InFilename)
			, Crc32(InCrc32)
			, Offset(InOffset)
			, Time(InTime)
			, CompressedSize(0)
			, UnCompressedSize(0)
			, bIsCompress(false)
		{
		}
	};

	TArray<FFileEntry> Files;

	TArray<uint8> Buffer;
	IFileHandle* File;
	EZipArchiveOptions ZipOptions;
	
	inline void Write(uint16 V) { Write((void*)&V, sizeof(V)); }
	inline void Write(uint32 V) { Write((void*)&V, sizeof(V)); }
	inline void Write(uint64 V) { Write((void*)&V, sizeof(V)); }
	inline void Write(void* Src, uint64 Size)
	{
		if (Size)
		{
			void* Dst = &Buffer[Buffer.AddUninitialized(IntCastChecked<int32>(Size))];
			FMemory::Memcpy(Dst, Src, Size);
		}
	}
	inline uint64 Tell() { return (File ? File->Tell() : 0) + Buffer.Num(); }
	void Flush();

public:
	UE_API FZipArchiveWriter(IFileHandle* InFile, EZipArchiveOptions InZipOptions = EZipArchiveOptions::None);
	UE_API ~FZipArchiveWriter();

	UE_API void AddFile(const FString& Filename, TConstArrayView<uint8> Data, const FDateTime& Timestamp);
	UE_API void AddFile(const FString& Filename, const TArray<uint8>& Data, const FDateTime& Timestamp);
};

#endif

#undef UE_API
