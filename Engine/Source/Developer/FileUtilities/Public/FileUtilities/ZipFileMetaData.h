// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Misc/Optional.h"

namespace UE::FileUtilities
{
struct FZipFileMetaData
{
	/** The timestamp of the file, if available. */
	TOptional<FDateTime> Timestamp;
};
}
